<?xml version="1.0" encoding="UTF-8"?>
<!DOCTYPE plist PUBLIC "-//Apple//DTD PLIST 1.0//EN" "http://www.apple.com/DTDs/PropertyList-1.0.dtd">
<plist version="1.0">
<dict>
	<key>CFBundleDevelopmentRegion</key>
	<string>en</string>
	<key>CFBundleExecutable</key>
	<string>$(EXECUTABLE_NAME)</string>
	<key>CFBundleIconFile</key>
	<string>__UE4_ICON_FILE__</string>
	<key>CFBundleIdentifier</key>
	<string>com.epicgames.$(PRODUCT_NAME:rfc1034identifier)</string>
	<key>CFBundleInfoDictionaryVersion</key>
	<string>6.0</string>
	<key>CFBundleName</key>
	<string>$(PRODUCT_NAME)</string>
	<key>CFBundlePackageType</key>
	<string>APPL</string>
	<key>CFBundleShortVersionString</key>
	<string>1.0</string>
	<key>CFBundleSignature</key>
	<string>????</string>
	<key>CFBundleVersion</key>
	<string>1</string>
	<key>LSMinimumSystemVersion</key>
	<string>$(MACOSX_DEPLOYMENT_TARGET)</string>
	<key>LSUIElement</key>
	<true/>
	<key>UE4AppToLaunch</key>
	<string>__UE4_APP_TO_LAUNCH__</string>
	<key>UE4CommandLine</key>
	<string>__UE4_COMMANDLINE__</string>
	<key>NSHumanReadableCopyright</key>
	<string>Copyright © Epic Games, Inc. All rights reserved.</string>
</dict>
</plist>
