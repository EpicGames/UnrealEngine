// Copyright Epic Games, Inc. All Rights Reserved.

#import <Cocoa/Cocoa.h>

int main(int argc, const char * argv[])
{
	NSWorkspace* Workspace = [NSWorkspace sharedWorkspace];
	NSBundle* Bundle = [NSBundle mainBundle];
	NSDictionary* Info = [Bundle infoDictionary];
	NSString* AppToLaunch = [Info valueForKey:@"UE4AppToLaunch"];
	NSString* CommandLine = [Info valueForKey:@"UE4CommandLine"];

	if (!AppToLaunch)
	{
		AppToLaunch = [[[Bundle bundlePath] lastPathComponent] stringByDeletingPathExtension];
	}

	if ([AppToLaunch hasPrefix:@"com."])
	{
		AppToLaunch = [Workspace absolutePathForAppBundleWithIdentifier:AppToLaunch];
	}
	else if (![AppToLaunch hasPrefix:@"/"])
	{
		NSString* BaseDir = [[Bundle bundlePath] stringByDeletingLastPathComponent];
		NSString* FullPath = [BaseDir stringByAppendingPathComponent:AppToLaunch];
		NSFileManager* FileManager = [NSFileManager defaultManager];
		if ([FileManager fileExistsAtPath:FullPath])
		{
			AppToLaunch = FullPath;
		}
		else
		{
			FullPath = [BaseDir stringByAppendingPathComponent:[NSString stringWithFormat:@"%@/Binaries/Mac/%@.app", AppToLaunch, AppToLaunch]];
			if ([FileManager fileExistsAtPath:FullPath])
			{
				AppToLaunch = FullPath;
			}
			else
			{
				NSAlert *Alert = [[NSAlert alloc] init];
				Alert.messageText = @"Couldn't find the app bundle to launch.";
				[Alert setInformativeText:AppToLaunch];
				[Alert runModal];
				[Alert release];
				return 0;
			}
		}
	}

	if (AppToLaunch)
	{
		NSMutableArray *Arguments = [[NSMutableArray new] autorelease];

		if (CommandLine && [CommandLine length] > 0)
		{
			NSArray* ArgsArray = [CommandLine componentsSeparatedByString:@" "];

			NSString* MultiPartArg = @"";
			for (NSString* Item in ArgsArray)
			{
				if ([MultiPartArg length] == 0)
				{
					if (([Item hasPrefix:@"\""] && ![Item hasSuffix:@"\""]) // check for a starting quote but no ending quote, excludes quoted single arguments
						|| ([Item rangeOfString:@"=\""].location != NSNotFound && ![Item hasSuffix:@"\""]) // check for quote after =, but no ending quote, this gets arguments of the type -blah="string string string"
						|| [Item hasSuffix:@"=\""]) // check for ending quote after =, this gets arguments of the type -blah=" string string string "
					{
						MultiPartArg = [NSString stringWithString:Item];
					}
					else
					{
						NSString* Arg;
						if ([Item rangeOfString:@"=\""].location != NSNotFound)
						{
							Arg = [[Item stringByReplacingOccurrencesOfString:@"=\"" withString:@"="] stringByTrimmingCharactersInSet:[NSCharacterSet characterSetWithCharactersInString:@"\""]];
						}
						else
						{
							Arg = [Item stringByTrimmingCharactersInSet:[NSCharacterSet characterSetWithCharactersInString:@"\""]];
						}
						[Arguments addObject:Arg];
					}
				}
				else
				{
					MultiPartArg = [MultiPartArg stringByAppendingString:@" "];
					MultiPartArg = [MultiPartArg stringByAppendingString:Item];
					if ([Item hasSuffix:@"\""])
					{
						NSString* Arg;
						if ([MultiPartArg hasPrefix:@"\""])
						{
							Arg = [MultiPartArg stringByTrimmingCharactersInSet:[NSCharacterSet characterSetWithCharactersInString:@"\""]];
						}
						else
						{
							Arg = [MultiPartArg stringByReplacingOccurrencesOfString:@"\"" withString:@""];
						}
						[Arguments addObject:Arg];
						MultiPartArg = @"";
					}
				}
			}
		}

		for (int Index = 1; Index < argc; Index++)
		{
			[Arguments addObject:[NSString stringWithUTF8String:argv[Index]]];
		}

		[Workspace launchApplicationAtURL:[NSURL fileURLWithPath:AppToLaunch]
								  options:NSWorkspaceLaunchDefault
							configuration:[NSDictionary dictionaryWithObject:Arguments forKey:NSWorkspaceLaunchConfigurationArguments]
									error:nil];
	}

	return 0;
}
