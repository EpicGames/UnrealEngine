// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DsymExporter : ModuleRules
{
	public DsymExporter( ReadOnlyTargetRules Target ) : base(Target)
	{
		PrivateIncludePathModuleNames.Add( "Launch" );
		bRequiresPlatformSDK = true;
	
		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{ 
				"Core",
				"ApplicationCore",
				"Projects"
			}
			);
	}
}
