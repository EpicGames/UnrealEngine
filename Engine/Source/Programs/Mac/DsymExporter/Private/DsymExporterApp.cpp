// Copyright Epic Games, Inc. All Rights Reserved.

#include "DsymExporterApp.h"
#include "Misc/Paths.h"

#include "GenericPlatform/GenericPlatformSymbolication.h"
#if PLATFORM_MAC
#include "Apple/ApplePlatformSymbolication.h"
#endif
#include "Serialization/Archive.h"

int32 RunDsymExporter(int32 ArgC, TCHAR* Argv[])
{
	// Make sure we have at least a single parameter
	if( ArgC < 2 )
	{
		UE_LOGF( LogInit, Error, "DsymExporter - not enough parameters." );
		UE_LOGF( LogInit, Error, " ... usage: DsymExporter [-UUID=ID] <Mach-O Binary Path> [Output Folder]" );
        UE_LOGF( LogInit, Error, "[-UUID=ID]: This is the UUID of the Mach-O Binary at the provided path. This for use by IOS because Core Symbolication is not properly finding it." );
		UE_LOGF( LogInit, Error, "<Mach-O Binary Path>: This is an absolute path to a Mach-O binary containing symbols, which may be the payload binary within an application, framework or dSYM bundle, an executable or dylib." );
		UE_LOGF( LogInit, Error, "[Output Folder]: The folder to write the new symbol database to, the database will take the filename of the input plus the .udebugsymbols extension." );
		return 1;
	}
	
#if PLATFORM_MAC
	FApplePlatformSymbolication::EnableCoreSymbolication(true);
#endif
	
	FPlatformSymbolDatabase Symbols;
    FString PathArg = Argv[1];
    FString SigArg;
    bool bHaveSigArg = false;
    if (PathArg[0] == L'-')
    {
        SigArg = Argv[1];
        PathArg = Argv[2];
        bHaveSigArg = true;
    }
	FString Signature = SigArg.RightChop(6);
	bool bOK = FPlatformSymbolication::LoadSymbolDatabaseForBinary(TEXT(""), PathArg, Signature, {}, Symbols);
	if(bOK)
	{
		FString OutputFolder = FPaths::GetPath(Argv[1]);
		if(ArgC == 3 && !bHaveSigArg)
		{
			OutputFolder = Argv[2];
		}
        else if (ArgC == 4)
        {
            OutputFolder = Argv[3];
        }
		
		if(FPlatformSymbolication::SaveSymbolDatabaseForBinary(OutputFolder, FPaths::GetBaseFilename(PathArg), Signature, Symbols))
		{
			return 0;
		}
		else
		{
			return 1;
		}
	}
	else
	{
		UE_LOGF( LogInit, Error, "DsymExporter - unable to parse debug symbols for Mach-O file." );
		return 1;
	}
}


