// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Export symbol data from Apple's .dSYM format to something that can be parsed on non-Apple platforms
int32 RunDsymExporter(int32 ArgC, TCHAR* Argv[]);


