// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealAtoSApp.h"

#include "CoreMinimal.h"
#include "RequiredProgramMainCPPInclude.h"
#include "HAL/ExceptionHandling.h"

IMPLEMENT_APPLICATION( UnrealAtoS, "UnrealAtoS" )

 // Main entry point to the application
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FPlatformMisc::SetGracefulTerminationHandler();
	FPlatformMisc::SetCrashHandler(nullptr);
	
	GEngineLoop.PreInit( ArgC, ArgV );

	const int32 Result = RunUnrealAtoS( ArgC, ArgV );
	
	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	
	return Result;
}
