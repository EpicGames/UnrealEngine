// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealAtoSApp.h"
#include "Misc/Paths.h"
#include "HAL/PlatformMisc.h"

#include "GenericPlatform/GenericPlatformSymbolication.h"
#if PLATFORM_MAC
#include "Apple/ApplePlatformSymbolication.h"
#endif

static void ShowUsage()
{
	UE_LOGF( LogInit, Error, "UnrealAtoS - not enough parameters." );
	UE_LOGF( LogInit, Error, " ... usage: UnrealAtoS <binary-image> [-s <binary-signature>] [-d <usymbol-path>] [-l <module-load-address>] <program-counter>" );
}

int32 RunUnrealAtoS(int32 ArgC, TCHAR* Argv[])
{
	// Make sure we have at least a single parameter
	if( ArgC < 4 )
	{
		ShowUsage();
		return 1;
	}
	
#if PLATFORM_MAC
	FApplePlatformSymbolication::EnableCoreSymbolication(true);
#endif

	TCHAR* ModulePath = Argv[1];
	TCHAR* ModuleSig = nullptr;
	TCHAR* SymbolPath = nullptr;
	uint64 ModuleBase = 0;
	uint64 ProgramCounter = FCStringWide::Strtoui64(Argv[ArgC - 1], nullptr, 16);
	
	for(int32 ArgI = 2; ArgI < ArgC - 1;)
	{
		if(FCStringWide::Strcmp(Argv[ArgI], TEXT("-d")) == 0)
		{
			if(ArgC - 1 > ++ArgI)
			{
				SymbolPath = Argv[ArgI++];
			}
			else
			{
				ShowUsage();
				return 1;
			}
		}
		else if(FCStringWide::Strcmp(Argv[ArgI], TEXT("-l")) == 0)
		{
			if(ArgC - 1 > ++ArgI)
			{
				ModuleBase = FCStringWide::Strtoui64(Argv[ArgI++], nullptr, 16);
			}
			else
			{
				ShowUsage();
				return 1;
			}
		}
		else if(FCStringWide::Strcmp(Argv[ArgI], TEXT("-s")) == 0)
		{
			if(ArgC - 1 > ++ArgI)
			{
				ModuleSig = Argv[ArgI++];
			}
			else
			{
				ShowUsage();
				return 1;
			}
		}
	}
	
	if(ModulePath && ProgramCounter)
	{
		uint64 Address = ProgramCounter > ModuleBase ? (ProgramCounter - ModuleBase) : ProgramCounter;
		FProgramCounterSymbolInfo Info;
		
		bool bOK = false;
		
		FString ModuleSignature = ModuleSig;
		FPlatformSymbolDatabase SymbolDb;
		if(FPlatformSymbolication::LoadSymbolDatabaseForBinary(FString(SymbolPath), ModulePath, ModuleSignature, {}, SymbolDb))
		{
			bOK = FPlatformSymbolication::SymbolInfoForStrippedSymbol(SymbolDb, ProgramCounter, ModuleBase, ModuleSignature, Info);
		}

		if(bOK)
		{
			FString FunctionName = Info.FunctionName;
			if(FunctionName.IsEmpty())
			{
				FunctionName = TEXT("[Unknown]");
			}
			
			FString ModuleName = FPaths::GetCleanFilename(Info.ModuleName);
			if(ModuleName.IsEmpty())
			{
				ModuleName = TEXT("[Unknown]");
			}
			
			FString Output;
			FString FileName = FPaths::GetCleanFilename(Info.Filename);
			if(!FileName.IsEmpty() && Info.LineNumber > 0)
			{
				Output = FString::Printf(TEXT("%s (in %s) (%s:%d)\n"), *FunctionName, *ModuleName, *FileName, Info.LineNumber);
			}
			else
			{
				Output = FString::Printf(TEXT("%s (in %s) + %d\n"), *FunctionName, *ModuleName, Info.SymbolDisplacement);
			}
		
			FPlatformMisc::LocalPrint(*Output);
			return 0;
		}
		else
		{
			UE_LOGF( LogInit, Error, "UnrealAtoS - no such symbol." );
			return 1;
		}
	}
	else
	{
		UE_LOGF( LogInit, Error, "UnrealAtoS - invalid arguments." );
		return 1;
	}
}


