// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Mac")]
public class UnrealAtoSTarget : TargetRules
{
	public UnrealAtoSTarget( TargetInfo Target ) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "UnrealAtoS";

		bBuildDeveloperTools = false;

		// DsymExporter doesn't ever compile with the engine linked in
		bCompileAgainstEngine = false;

		// DsymExporter has no exports, so no need to verify that a .lib and .exp file was emitted by the linker.
		bHasExports = false;

        bCompileAgainstCoreUObject = false;

		// Do NOT produce additional console app exe
		bIsBuildingConsoleApplication = true;
	}
}
