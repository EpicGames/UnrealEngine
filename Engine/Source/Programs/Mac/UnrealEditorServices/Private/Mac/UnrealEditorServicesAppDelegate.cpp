// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealEditorServicesAppDelegate.h"
#include "DesktopPlatformModule.h"

@implementation FUnrealEditorServicesAppDelegate

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	[NSApp setServicesProvider:self];
}

- (NSBundle*)defaultAppForUProjectFile:(NSURL*)FileURL
{
	CFURLRef FileDefaultAppURL = NULL;
	NSBundle* FileDefaultAppBundle = NULL;
	OSStatus Status = LSGetApplicationForURL((__bridge CFURLRef)FileURL, kLSRolesAll, NULL, &FileDefaultAppURL);
	if (Status == noErr)
	{
		FileDefaultAppBundle = [NSBundle bundleWithURL:(__bridge NSURL*)FileDefaultAppURL];
		CFRelease(FileDefaultAppURL);

		if (![self isAppValidForUProjectFiles:FileDefaultAppBundle])
		{
			FileDefaultAppBundle = NULL;
		}
	}
	return FileDefaultAppBundle;
}

- (NSBundle*)defaultAppForUProjectFiles
{
	CFURLRef GlobalDefaultAppURL = NULL;
	NSBundle* GlobalDefaultAppBundle = NULL;
	OSStatus Status = LSGetApplicationForInfo(kLSUnknownType, kLSUnknownCreator, CFSTR("uproject"), kLSRolesAll, NULL, &GlobalDefaultAppURL);
	if (Status == noErr)
	{
		GlobalDefaultAppBundle = [NSBundle bundleWithURL:(__bridge NSURL*)GlobalDefaultAppURL];
		CFRelease(GlobalDefaultAppURL);

		if (![self isAppValidForUProjectFiles:GlobalDefaultAppBundle])
		{
			GlobalDefaultAppBundle = NULL;
		}
	}
	return GlobalDefaultAppBundle;
}

- (NSBundle*)recommendedAppForUProjectFile:(NSURL*)FileURL
{
	FString EngineAssociation;
	if (!FDesktopPlatformModule::Get()->GetEngineIdentifierForProject([FileURL path], EngineAssociation))
	{
		EngineAssociation = TEXT("4.0");
	}

	TMap<FString, FString> Installations;
	FDesktopPlatformModule::Get()->EnumerateEngineInstallations(Installations);
	for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
	{
		if (Iter.Key() == EngineAssociation)
		{
			FString BundlePath = Iter.Value() / TEXT("Engine/Binaries/Mac/UnrealEditor.app");
			return [NSBundle bundleWithPath:BundlePath.GetNSString()];
		}
	}

	return [NSBundle mainBundle];
}

- (NSString*)defaultEngineForUProjectFile:(NSURL*)FileURL
{
	NSString* FileDefaultEnginePath = [self enginePathForAppBundle:[self defaultAppForUProjectFile:FileURL]];
	NSString* GlobalDefaultEnginePath = [self enginePathForAppBundle:[self defaultAppForUProjectFiles]];
	if (!FileDefaultEnginePath || [FileDefaultEnginePath isEqualToString:GlobalDefaultEnginePath])
	{
		FileDefaultEnginePath = [self enginePathForAppBundle:[self recommendedAppForUProjectFile:FileURL]];
	}
	return FileDefaultEnginePath;
}

- (IBAction)onCancelButtonPressed:(id)Sender
{
	[NSApp stopModalWithCode:NSCancelButton];
	[Window close];
}

- (IBAction)onOKButtonPressed:(id)Sender
{
	[NSApp stopModalWithCode:NSOKButton];
	[Window close];
}

- (FString)showEngineSelectionWindowForURL:(NSURL*)FileURL
{
	TMap<FString, FString> Installations;
	FDesktopPlatformModule::Get()->EnumerateEngineInstallations(Installations);
	if (Installations.Num() == 0)
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Unreal Engine is not installed"), TEXT("Error"));
		return TEXT("");
	}

	NSString* GlobalDefaultEnginePath = [self enginePathForAppBundle:[self defaultAppForUProjectFiles]];
	NSString* RecommendedEnginePath = [self enginePathForAppBundle:[self recommendedAppForUProjectFile:FileURL]];
	NSString* SelectedEnginePath = [self defaultEngineForUProjectFile:FileURL];

	NSMenu* SubMenu = [NSMenu new];
	NSMenuItem* SelectedItem = NULL;

	if (GlobalDefaultEnginePath)
	{
		SelectedItem = [self addEnginePath:GlobalDefaultEnginePath toEditorMenu:SubMenu withDescription:@"(default)"];
	}

	if (GlobalDefaultEnginePath)
	{
		[SubMenu addItem:[NSMenuItem separatorItem]];
	}

	for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
	{
		NSString* EnginePath = Iter.Value().GetNSString();
		if (![EnginePath isEqualToString:GlobalDefaultEnginePath])
		{
			NSMenuItem* MenuItem = [self addEnginePath:EnginePath toEditorMenu:SubMenu withDescription:[EnginePath isEqualToString:RecommendedEnginePath] ? @"(recommended)" : NULL];
			if ([EnginePath isEqualToString:SelectedEnginePath])
			{
				SelectedItem = MenuItem;
			}
		}
	}

	Window = [[NSWindow alloc] initWithContentRect:NSMakeRect(0, 0, 433, 102) styleMask:NSTitledWindowMask backing:NSBackingStoreBuffered defer:NO];

	NSPopUpButton* EditorMenu = [[NSPopUpButton alloc] initWithFrame:NSMakeRect(18, 58, 398, 26) pullsDown:NO];
	[EditorMenu setMenu:SubMenu];
	[EditorMenu selectItem:SelectedItem];

	NSButton* OKButton = [NSButton new];
	[OKButton setButtonType:NSMomentaryLightButton];
	[OKButton setTitle:@"OK"];
	[OKButton setFrame:NSMakeRect(337, 13, 82, 32)];
	[OKButton setImagePosition:NSNoImage];
	[OKButton setBezelStyle:NSRoundedBezelStyle];
	[OKButton setFont:[NSFont systemFontOfSize:0]];
	[OKButton setAction:@selector(onOKButtonPressed:)];
	[OKButton setTarget:self];

	NSButton* CancelButton = [NSButton new];
	[CancelButton setButtonType:NSMomentaryLightButton];
	[CancelButton setTitle:@"Cancel"];
	[CancelButton setFrame:NSMakeRect(255, 13, 82, 32)];
	[CancelButton setImagePosition:NSNoImage];
	[CancelButton setBezelStyle:NSRoundedBezelStyle];
	[CancelButton setFont:[NSFont systemFontOfSize:0]];
	[CancelButton setAction:@selector(onCancelButtonPressed:)];
	[CancelButton setTarget:self];

	[[Window contentView] addSubview:EditorMenu];
	[[Window contentView] addSubview:OKButton];
	[[Window contentView] addSubview:CancelButton];

	[Window setTitle:@"Choose Unreal Editor..."];
	[Window setRestorable:NO];
	[Window disableSnapshotRestoration];

	[Window center];
	[NSApp activateIgnoringOtherApps:YES];

	NSInteger Result = [NSApp runModalForWindow:Window];
	if (Result == NSOKButton)
	{
		SelectedEnginePath = (NSString*)[[EditorMenu selectedItem] representedObject];
		for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
		{
			if ([Iter.Value().GetNSString() isEqualToString:SelectedEnginePath])
			{
				return Iter.Key();
			}
		}
	}

	return TEXT("");
}

- (NSString*)findEngineForUProjectFile:(NSURL*)FileURL
{
	// First, check if this file is associated to a specific build in Finder. If it is, we use this build no matter what's set in .uproject's EngineAssociation
	NSString* FileDefaultEnginePath = [self enginePathForAppBundle:[self defaultAppForUProjectFile:FileURL]];
	NSString* GlobalDefaultEnginePath = [self enginePathForAppBundle:[self defaultAppForUProjectFiles]];
	if (FileDefaultEnginePath && ![FileDefaultEnginePath isEqualToString:GlobalDefaultEnginePath] && [[NSFileManager defaultManager] fileExistsAtPath:FileDefaultEnginePath])
	{
		return FileDefaultEnginePath;
	}

	TMap<FString, FString> Installations;
	FDesktopPlatformModule::Get()->EnumerateEngineInstallations(Installations);

	// Now, read EngineAssociation from .uproject file and use the build that the project is associated with if it's installed
	FString EngineAssociation;
	if (FDesktopPlatformModule::Get()->GetEngineIdentifierForProject([FileURL path], EngineAssociation))
	{
		for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
		{
			if (Iter.Key() == EngineAssociation)
			{
				return Iter.Value().GetNSString();
			}
		}
	}

	// The build this file is associated with is not installed, so ask user to select one
	FString EngineID = [self showEngineSelectionWindowForURL:FileURL];
	if (!EngineID.IsEmpty())
	{
		return Installations[EngineID].GetNSString();
	}

	return nil;
}

- (BOOL)application:(NSApplication*)Application openFile:(NSString*)Filename
{
	NSURL* FileURL = [NSURL fileURLWithPath:Filename];
	NSString* EnginePath = [self findEngineForUProjectFile:FileURL];
	if (EnginePath)
	{
		NSURL* EditorBundleURL = [NSURL fileURLWithPath:[EnginePath stringByAppendingPathComponent:@"Engine/Binaries/Mac/UnrealEditor.app"]];
		if (EditorBundleURL)
		{
			NSDictionary* Configuration = [NSDictionary dictionaryWithObject:[NSArray arrayWithObjects:[FileURL path], nil] forKey:NSWorkspaceLaunchConfigurationArguments];
			NSError* LaunchError = NULL;
			NSRunningApplication* NewInstance = [[NSWorkspace sharedWorkspace] launchApplicationAtURL:EditorBundleURL options:(NSWorkspaceLaunchOptions)(NSWorkspaceLaunchAsync|NSWorkspaceLaunchNewInstance) configuration:Configuration error:&LaunchError];
			if (NewInstance)
			{
				return YES;
			}
		}
	}

	return NO;
}

- (void)launchGameService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error
{
	NSString* UnrealError = NULL;

	if ([[PBoard types] containsObject:NSFilenamesPboardType])
    {
		NSURL* FileURL = [NSURL fileURLWithPath: [[PBoard propertyListForType:NSFilenamesPboardType] objectAtIndex:0]];
		NSString* EnginePath = [self findEngineForUProjectFile:FileURL];
		if (EnginePath)
		{
			NSURL* EditorBundleURL = [NSURL fileURLWithPath:[EnginePath stringByAppendingPathComponent:@"Engine/Binaries/Mac/UnrealEditor.app"]];
			if (EditorBundleURL)
			{
				NSDictionary* Configuration = [NSDictionary dictionaryWithObject:[NSArray arrayWithObjects:[FileURL path], @"-game", nil] forKey:NSWorkspaceLaunchConfigurationArguments];
				NSError* LaunchError = NULL;
				NSRunningApplication* NewInstance = [[NSWorkspace sharedWorkspace] launchApplicationAtURL:EditorBundleURL options:(NSWorkspaceLaunchOptions)(NSWorkspaceLaunchAsync|NSWorkspaceLaunchNewInstance) configuration:Configuration error:&LaunchError];
				if (!NewInstance)
				{
					if (LaunchError)
					{
						UnrealError = [LaunchError localizedDescription];
					}
					else
					{
						UnrealError = @"Failed to run a copy of the game on this machine.";
					}
				}
			}
			else
			{
				UnrealError = @"No application to open the project file available.";
			}
		}
	}
	else
	{
		UnrealError = @"No valid project file selected.";
	}

	if (Error)
	{
		*Error = UnrealError;
	}
}

- (BOOL)canControlTerminal
{
	// This will verify if we can access the terminal and ask the user if necessary
	NSAppleEventDescriptor* targetAppEventDescriptor = [NSAppleEventDescriptor descriptorWithBundleIdentifier:@"com.apple.terminal"];
	OSStatus status = AEDeterminePermissionToAutomateTarget(targetAppEventDescriptor.aeDesc, typeWildCard, typeWildCard, true);
	return (status == noErr);
}

- (void)generateXcodeProjectService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error
{
	NSString* UnrealError = NULL;

	if ([[PBoard types] containsObject:NSFilenamesPboardType])
    {
		NSURL* FileURL = [NSURL fileURLWithPath: [[PBoard propertyListForType:NSFilenamesPboardType] objectAtIndex:0]];
		NSString* EnginePath = [self findEngineForUProjectFile:FileURL];
		if (EnginePath)
		{
			// Check it's a code project
			FString SourceDir = FPaths::GetPath(FString([[PBoard propertyListForType:NSFilenamesPboardType] objectAtIndex:0])) / TEXT("Source");
			if(!IPlatformFile::GetPlatformPhysical().DirectoryExists(*SourceDir))
			{
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("This project does not have any source code. You need to add C++ source files to the project from the Editor before you can generate project files."), TEXT("Error"));
				return;
			}

			NSString* ScriptPath = [EnginePath stringByAppendingPathComponent:@"Engine/Build/BatchFiles/Mac/GenerateProjectFiles.sh"];
			if (![[NSFileManager defaultManager] fileExistsAtPath:ScriptPath])
			{
				ScriptPath = [EnginePath stringByAppendingPathComponent:@"Engine/Build/BatchFiles/Mac/RocketGenerateProjectFiles.sh"];
			}

			if ([[NSFileManager defaultManager] fileExistsAtPath:ScriptPath])
			{
				if ([[NSWorkspace sharedWorkspace] launchAppWithBundleIdentifier:@"com.apple.terminal" options:0 additionalEventParamDescriptor:nil launchIdentifier:nil])
				{
					if ( [self canControlTerminal] )
					{
						NSString* FullFolderPath = [[ScriptPath stringByDeletingLastPathComponent] stringByResolvingSymlinksInPath];
						NSString* FullScriptPath = [ScriptPath stringByResolvingSymlinksInPath];
						NSString* Command = [NSString stringWithFormat:@"cd \"%@\" \n sh \"%@\" -project=\"%@\" -game\n logout\n", FullFolderPath, FullScriptPath, [FileURL path]];

						const char* UTF8Script = [Command UTF8String];

						const char *TerminalBundleID = "com.apple.terminal";

						// Build event
						AppleEvent Event, Reply;
						OSStatus Status = AEBuildAppleEvent(kAECoreSuite, kAEDoScript,
												typeApplicationBundleID,
												TerminalBundleID, strlen(TerminalBundleID),
												kAutoGenerateReturnID,
												kAnyTransactionID,
												&Event, NULL,
												"'----':utf8(@)", [Command length],
												UTF8Script);
						if (Status == noErr)
						{
							// Send event and check for any Apple Event Manager errors
							Status = AESendMessage(&Event, &Reply, kAENoReply, kAEDefaultTimeout);
							AEDisposeDesc(&Event);
							if (Status != noErr)
							{
								UnrealError = @"Couldn't tell Terminal to generate project files.";
							}
						}
						else
						{
							UnrealError = @"Couldn't tell Terminal to generate project files.";
						}
					}
					else
					{
						UnrealError = @"Didn't have permission to tell Terminal to generate project files.";
					}
				}
				else
				{
					UnrealError = @"Failed to open Terminal while trying to generate project files.";
				}
			}
			else
			{
				UnrealError = @"No application to generate project files available.";
			}
		}
	}
	else
	{
		UnrealError = @"No valid project file selected.";
	}

	if (Error)
	{
		*Error = UnrealError;
	}
}

- (NSMenuItem*)addEnginePath:(NSString*)EnginePath toEditorMenu:(NSMenu*)Menu withDescription:(NSString*)Suffix
{
	NSFileManager* FileManager = [NSFileManager defaultManager];
	const bool bIsPerforceBuild = [FileManager fileExistsAtPath:[EnginePath stringByAppendingPathComponent:@"Engine/Build/PerforceBuild.txt"]];
	const bool bIsSourceDistribution = [FileManager fileExistsAtPath:[EnginePath stringByAppendingPathComponent:@"Engine/Build/SourceDistribution.txt"]];

	NSString* EditorDescription = NULL;
	if (bIsPerforceBuild)
	{
		EditorDescription = @"Perforce Build";
	}
	else if (bIsSourceDistribution)
	{
		EditorDescription = @"Source Build";
	}
	else
	{
		EditorDescription = @"Binary Build";
		NSBundle* EditorBundle = [NSBundle bundleWithPath:[EnginePath stringByAppendingPathComponent:@"Engine/Binaries/Mac/UnrealEditor.app"]];
		if (EditorBundle)
		{
			EditorDescription = (NSString*)[[EditorBundle infoDictionary] objectForKey:@"CFBundleVersion"];
		}
	}
	if (Suffix && [Suffix length] > 0)
	{
		EditorDescription = [EditorDescription stringByAppendingString:[NSString stringWithFormat:@" %@", Suffix]];
	}
	EditorDescription = [EditorDescription stringByAppendingString:[@" at " stringByAppendingString:EnginePath]];

	NSMenuItem* MenuItem = [NSMenuItem new];
	[MenuItem setTitle:EditorDescription];
	[MenuItem setRepresentedObject:EnginePath];
	[Menu addItem:MenuItem];

	return MenuItem;
}

- (BOOL)isAppValidForUProjectFiles:(NSBundle*)AppBundle
{
	return AppBundle && ([[AppBundle bundleIdentifier] isEqualToString:@"com.epicgames.UnrealEditor"] || [[AppBundle bundleIdentifier] isEqualToString:@"com.epicgames.UnrealEditorServices"])
					 && [[[AppBundle bundlePath] stringByDeletingLastPathComponent] hasSuffix:@"Engine/Binaries/Mac"]
					 && [[AppBundle bundlePath] rangeOfString:@".app/Contents/UE"].location == NSNotFound;
}

- (NSString*)enginePathForAppBundle:(NSBundle*)AppBundle
{
	return AppBundle ? [[[[[AppBundle bundlePath] stringByDeletingLastPathComponent] stringByDeletingLastPathComponent] stringByDeletingLastPathComponent] stringByDeletingLastPathComponent] : NULL;
}

- (void)switchUnrealEngineVersionService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error
{
	if ([[PBoard types] containsObject:NSFilenamesPboardType])
    {
		NSURL* FileURL = [NSURL fileURLWithPath: [[PBoard propertyListForType:NSFilenamesPboardType] objectAtIndex:0]];
		FString EngineID = [self showEngineSelectionWindowForURL:FileURL];
		if (!EngineID.IsEmpty() && !FDesktopPlatformModule::Get()->SetEngineIdentifierForProject([FileURL path], EngineID))
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't set association for project. Check the file is writeable."), TEXT("Error"));
		}
	}
}

@end
