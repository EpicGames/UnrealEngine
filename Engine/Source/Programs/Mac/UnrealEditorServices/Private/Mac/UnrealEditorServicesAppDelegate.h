// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Mac/MacSystemIncludes.h"
#include "Misc/Paths.h"

@interface FUnrealEditorServicesAppDelegate : NSObject <NSApplicationDelegate, NSFileManagerDelegate>
{
	NSWindow* Window;
}

- (IBAction)onCancelButtonPressed:(id)Sender;
- (IBAction)onOKButtonPressed:(id)Sender;

- (void)launchGameService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error;
- (void)generateXcodeProjectService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error;
- (void)switchUnrealEngineVersionService:(NSPasteboard *)PBoard userData:(NSString *)UserData error:(NSString **)Error;

@end
