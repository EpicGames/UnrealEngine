// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/ScopeAutoreleasePool.h"
#include "CoreMinimal.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Modules/ModuleManager.h"
#include "UnrealEditorServicesAppDelegate.h"
#include "Misc/CommandLine.h"
#include "Mac/MacSystemIncludes.h"

IMPLEMENT_APPLICATION(UnrealEditorServices, "UnrealEditorServices");

int main(int argc, char *argv[])
{
	FCommandLine::Set(TEXT(""));
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
	SCOPED_AUTORELEASE_POOL;
	[NSApplication sharedApplication];
	[NSApp setDelegate:[FUnrealEditorServicesAppDelegate new]];
	[NSApp run];
	return 0;
}
