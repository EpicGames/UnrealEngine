// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_APPLICATION(UnrealEditorServices, "UnrealEditorServices");

int WINAPI WinMain(HINSTANCE hInstance, HINSTANCE hPrevInstance, LPSTR lpCmdLine, int ShowCmd)
{
	return 0;
}
