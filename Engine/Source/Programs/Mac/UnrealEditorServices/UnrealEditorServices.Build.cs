// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealEditorServices : ModuleRules
{
	public UnrealEditorServices(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"DesktopPlatform",
				"Json",
				"Projects",
			}
		);
	}
}
