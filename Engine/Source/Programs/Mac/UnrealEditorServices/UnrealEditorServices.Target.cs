// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Mac")]
public class UnrealEditorServicesTarget : TargetRules
{
	public UnrealEditorServicesTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "UnrealEditorServices";

		bBuildDeveloperTools = false;

		bCompileICU = false;
		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// We still need to support old versions of the engine that are compatible with OS X 10.9
		bEnableOSX109Support = true;
	}
}
